// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EditorTelemetry : ModuleRules
	{
		public EditorTelemetry(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"Analytics",
					"TelemetryUtils",
					"AssetRegistry",
					"Engine",
					"UnrealEd",
					"HTTP",
					"Json",
					"StudioTelemetry",
					"DerivedDataCache",
					"Zen",
					"ContentBrowser",
					"ContentBrowserData",
				}
			);

			PrivateIncludePathModuleNames.AddRange([
				"DataValidation"
			]);
		}
	}
}
