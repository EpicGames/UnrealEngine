// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTelemetry.h"

#include "Algo/Copy.h"
#include "AnalyticsTracer.h"
#include "AssetRegistry/AssetRegistryTelemetry.h"
#include "CollectionManagerModule.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserTelemetry.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "DataValidationTelemetry.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheUsageStats.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/AssetManager.h"
#include "Experimental/ZenServerInterface.h"
#include "FileHelpers.h"
#include "HttpManager.h"
#include "HttpModule.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "ProfilingDebugging/CookStats.h"
#include "ShaderStats.h"
#include "StudioTelemetry.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TelemetryRouter.h"
#include "Templates/Greater.h"
#include "UObject/ICookInfo.h"
#include "UnrealEdGlobals.h"
#include "Virtualization/VirtualizationSystem.h"

namespace Private
{
	const FName ContentBrowserModuleName = TEXT("ContentBrowser");
	
	// Json writer subclass to allow us to avoid using a SharedPtr to write basic Json.
	typedef TCondensedJsonPrintPolicy<TCHAR> FPrintPolicy;
	class FAnalyticsJsonWriter : public TJsonStringWriter<FPrintPolicy>
	{
	public:
		explicit FAnalyticsJsonWriter(FString* Out) : TJsonStringWriter<FPrintPolicy>(Out, 0)
		{
		}
	};
}

namespace UE::Telemetry::DataValidation
{
	inline const TCHAR* LexToString(EDataValidationResult Result)
	{
		switch (Result)
		{
			default:
			case EDataValidationResult::NotValidated: return TEXT("NotValidated");
			case EDataValidationResult::Invalid: return TEXT("Invalid");
			case EDataValidationResult::Valid: return TEXT("Valid");
		}
	}

	inline const TCHAR* LexToString(EDataValidationUsecase Usecase)
	{
		switch (Usecase)
		{
			default:
			case EDataValidationUsecase::None: return TEXT("None");
			case EDataValidationUsecase::Manual: return TEXT("Manual");
			case EDataValidationUsecase::Commandlet: return TEXT("Commandlet");
			case EDataValidationUsecase::Save: return TEXT("Save");
			case EDataValidationUsecase::PreSubmit: return TEXT("PreSubmit");
			case EDataValidationUsecase::Script: return TEXT("Script");
		}
	}

	inline const TCHAR* LexToString(EValidationMode Mode)
	{
		switch (Mode)
		{
			default:
			case EValidationMode::Unknown: return TEXT("Unknown");
			case EValidationMode::Editor: return TEXT("Editor");
			case EValidationMode::Commandlet: return TEXT("Commandlet");
		}
	}
}

const TCHAR* LexToString(ECollectionTelemetryAssetAddedWorkflow Enum)
{
	switch(Enum)
	{
	case ECollectionTelemetryAssetAddedWorkflow::ContextMenu: return TEXT("ContextMenu");
	case ECollectionTelemetryAssetAddedWorkflow::DragAndDrop: return TEXT("DragAndDrop");
	default: return TEXT("");
	}
}
	
const TCHAR* LexToString(ECollectionTelemetryAssetRemovedWorkflow Enum)
{
	switch(Enum)
	{
	case ECollectionTelemetryAssetRemovedWorkflow::ContextMenu: return TEXT("ContextMenu");
	default: return TEXT("");
	}
}

template<typename T>
FString AnalyticsOptionalToStringOrNull(const TOptional<T>& Opt)
{
	return Opt.IsSet() ? AnalyticsConversionToString(Opt.GetValue()) : FString(TEXT("null"));
}

FEditorTelemetry& FEditorTelemetry::Get()
{
	static FEditorTelemetry StudioTelemetryEditorInstance = FEditorTelemetry();
	return StudioTelemetryEditorInstance;
}

void FEditorTelemetry::RecordEvent_Cooking(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if ENABLE_COOK_STATS

	TMap<FString,FAnalyticsEventAttribute> CookAttributes;
	
	// Sends each cook stat to the studio analytics system.
	auto GatherAnalyticsAttributes = [&CookAttributes](const FString& StatName, const TArray<FCookStatsManager::StringKeyValue>& StatAttributes)
	{
		auto IsJsonFragment =
			[](const FString& MaybeJsonStr)
			{
				if (MaybeJsonStr.Len() >= 2 && 
					((MaybeJsonStr[0] == TEXT('{') && MaybeJsonStr[MaybeJsonStr.Len() - 1] == TEXT('}')) || (MaybeJsonStr[0] == TEXT('[') && MaybeJsonStr[MaybeJsonStr.Len() - 1] == TEXT(']')))
					)
				{
					FString MaybeJsonStrToParse = MaybeJsonStr;
					return UE::Json::ParseInPlace(MaybeJsonStrToParse).HasValue();
				}
				return false;
			};

		for (const FCookStatsManager::StringKeyValue& Attr : StatAttributes)
		{
			FString FormattedAttrName = StatName;
			FormattedAttrName += TEXT("_");
			FormattedAttrName += Attr.Key;
			FormattedAttrName.ReplaceCharInline(TEXT('.'), TEXT('_'));

			if (!CookAttributes.Contains(FormattedAttrName))
			{
				if (Attr.Value.IsNumeric())
				{
					CookAttributes.Emplace(FormattedAttrName, FAnalyticsEventAttribute(FormattedAttrName, FCString::Atof(*Attr.Value)));
				}
				else if (IsJsonFragment(Attr.Value))
				{
					CookAttributes.Emplace(FormattedAttrName, FAnalyticsEventAttribute(FormattedAttrName, FJsonFragment(CopyTemp(Attr.Value))));
				}
				else
				{
					CookAttributes.Emplace(FormattedAttrName, FAnalyticsEventAttribute(FormattedAttrName, Attr.Value));
				}
			}
		}
	};

	// Now actually grab the stats 
	FCookStatsManager::LogCookStats(GatherAnalyticsAttributes);

	// Build and send the full Cook event
	{
		const int SchemaVersion = 4;

		TArray<FAnalyticsEventAttribute> FullEventAttributes = Attributes;

		FullEventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		FullEventAttributes.Emplace(TEXT("Context"), Context);

		// Add the values to the full cook event attributes
		for (TMap<FString, FAnalyticsEventAttribute>::TConstIterator it(CookAttributes); it; ++it)
		{
			FullEventAttributes.Emplace((*it).Value);
		}

		// Append the cook stats
		FShaderStatsFunctions::GatherShaderAnalytics(FullEventAttributes);

		// Record the full cook event
		FStudioTelemetry::Get().RecordEvent(TEXT("Core.Cooking"), FullEventAttributes);
	}

	// Build and send the summarized Cook event
	{
		const int SchemaVersion = 1;

		TArray<FAnalyticsEventAttribute> SummaryEventAttributes = Attributes;

		SummaryEventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		SummaryEventAttributes.Emplace(TEXT("Context"), Context);

		// Add the values to the summary cook event attributes
		for (TMap<FString, FAnalyticsEventAttribute>::TConstIterator it(CookAttributes); it; ++it)
		{
			const FString& KeyName = (*it).Key;

			// Filter the Cook attributes to the ones we want to send in the summary event
			if ( KeyName.Find("Cook_Profile_") != INDEX_NONE || KeyName.Find("CookOnTheFlyServer_") != INDEX_NONE || KeyName.Find("Package_") != INDEX_NONE || KeyName.Find("UnversionedProperties_") != INDEX_NONE )
			{
				SummaryEventAttributes.Emplace((*it).Value);
			}
		}

		// Record the summary cook event
		FStudioTelemetry::Get().RecordEvent(TEXT("Core.Cooking.Summary"), SummaryEventAttributes);
	}

#endif
}

void FEditorTelemetry::RecordEvent_Loading(const FString& Context, double LoadingSeconds, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	const int SchemaVersion = 5;

	TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

	EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
	EventAttributes.Emplace(TEXT("Context"), Context);
	EventAttributes.Emplace(TEXT("LoadingName"), Context);
	EventAttributes.Emplace(TEXT("LoadingSeconds"), LoadingSeconds);
	
	FStudioTelemetry::Get().RecordEvent(TEXT("Core.Loading"), EventAttributes);
}

void FEditorTelemetry::RecordEvent_DDCResource(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if ENABLE_COOK_STATS
	// Gather the latest resource stats
	TArray<FDerivedDataCacheResourceStat> ResourceStats;

	GatherDerivedDataCacheResourceStats(ResourceStats);

	const int SchemaVersion = 4;

	// Send a resource event per asset type
	for (const FDerivedDataCacheResourceStat& Stat : ResourceStats)
	{
		const double TotalTimeSec = Stat.BuildTimeSec + Stat.LoadTimeSec;
		const int64 TotalSizeMB = Stat.BuildSizeMB + Stat.LoadSizeMB;

		if (Stat.AssetType.IsEmpty() || Stat.TotalCount==0)
		{
			// Empty asset type or nothing was built or loaded for this type
			continue;
		}
	
		TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

		EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		EventAttributes.Emplace(TEXT("Context"), Context);
		EventAttributes.Emplace(TEXT("AssetType"), Stat.AssetType);
		EventAttributes.Emplace(TEXT("Load_Count"), Stat.LoadCount);
		EventAttributes.Emplace(TEXT("Load_TimeSec"), Stat.LoadTimeSec);
		EventAttributes.Emplace(TEXT("Load_SizeMB"), Stat.LoadSizeMB);
		EventAttributes.Emplace(TEXT("Build_Count"), Stat.BuildCount);
		EventAttributes.Emplace(TEXT("Build_TimeSec"), Stat.BuildTimeSec);
		EventAttributes.Emplace(TEXT("Build_SizeMB"), Stat.BuildSizeMB);
		EventAttributes.Emplace(TEXT("Total_Count"), Stat.TotalCount);
		EventAttributes.Emplace(TEXT("Total_TimeSec"), TotalTimeSec);
		EventAttributes.Emplace(TEXT("Total_SizeMB"), TotalSizeMB);
		EventAttributes.Emplace(TEXT("Efficiency"), Stat.Efficiency);
		EventAttributes.Emplace(TEXT("Thread_TimeSec"), Stat.GameThreadTimeSec);

		FStudioTelemetry::Get().RecordEvent(TEXT("Core.DDC.Resource"), EventAttributes);
	}
#endif			
}


void FEditorTelemetry::RecordEvent_DDCSummary(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if ENABLE_COOK_STATS
	const int SchemaVersion = 4;

	TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

	EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
	EventAttributes.Emplace(TEXT("Context"), Context);

	// Gather the summary stats
	FDerivedDataCacheSummaryStats SummaryStats;

	GatherDerivedDataCacheSummaryStats(SummaryStats);

	// Append to the attributes
	for (const FDerivedDataCacheSummaryStat& Stat : SummaryStats.Stats)
	{
		FString AttributeName = Stat.Key.Replace(TEXT("."), TEXT("_"));

		if (Stat.Value.IsNumeric())
		{
			EventAttributes.Emplace(AttributeName, FCString::Atof(*Stat.Value));
		}
		else
		{
			EventAttributes.Emplace(AttributeName, Stat.Value);
		}
	}

	FStudioTelemetry::Get().RecordEvent(TEXT("Core.DDC.Summary"), EventAttributes);
#endif			
}

void FEditorTelemetry::RecordEvent_DDCVerification(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	const int SchemaVersion = 1;

	FDerivedDataCacheVerificationStats DDCVerifyStats;
	GetDerivedDataCacheRef().GatherVerificationStats(DDCVerifyStats);

	for (const FDerivedDataCacheVerificationStat& Stat : DDCVerifyStats.Entries)
	{
		if (Stat.RecordsTested > 0 || Stat.ValuesTested > 0)
		{
			TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

			EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
			EventAttributes.Emplace(TEXT("Context"), Context);
			EventAttributes.Emplace(TEXT("Bucket"), Stat.BucketName);
			EventAttributes.Emplace(TEXT("RecordsTested"), Stat.RecordsTested);
			EventAttributes.Emplace(TEXT("RecordsMatchedCache"), Stat.RecordsMatchedCache);
			EventAttributes.Emplace(TEXT("RecordsNotInCache"), Stat.RecordsNotInCache);
			EventAttributes.Emplace(TEXT("ValuesTested"), Stat.ValuesTested);
			EventAttributes.Emplace(TEXT("ValuesTestedBytes"), Stat.ValuesTestedBytes);
			EventAttributes.Emplace(TEXT("ValuesMatchedCache"), Stat.ValuesMatchedCache);
			EventAttributes.Emplace(TEXT("ValuesMatchedCacheBytes"), Stat.ValuesMatchedCacheBytes);
			EventAttributes.Emplace(TEXT("ValuesNotInCache"), Stat.ValuesNotInCache);
			EventAttributes.Emplace(TEXT("ValuesNotInCacheBytes"), Stat.ValuesNotInCacheBytes);
			EventAttributes.Emplace(TEXT("ValuesChanged"), Stat.ValuesChanged);
			EventAttributes.Emplace(TEXT("ValuesChangedBytes"), Stat.ValuesChangedBytes);
			EventAttributes.Emplace(TEXT("ValuesNotInRequest"), Stat.ValuesNotInRequest);
			EventAttributes.Emplace(TEXT("ValuesNotInRequestBytes"), Stat.ValuesNotInRequestBytes);
			FStudioTelemetry::Get().RecordEvent(TEXT("Core.DDC.Verify"), EventAttributes);
		}
	}
}

void FEditorTelemetry::RecordEvent_Zen(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	// Gather Zen analytics
	if (UE::Zen::IsDefaultServicePresent())
	{
		const int SchemaVersion = 2;

		TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

		EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		EventAttributes.Emplace(TEXT("Context"), Context);

		UE::Zen::GetDefaultServiceInstance().GatherAnalytics(EventAttributes);
		FStudioTelemetry::Get().RecordEvent(TEXT("Core.Zen"), EventAttributes);
	}
}

void FEditorTelemetry::RecordEvent_VirtualAssets(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (UE::Virtualization::IVirtualizationSystem::Get().IsEnabled())
	{
		const int SchemaVersion = 2;

		TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

		EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		EventAttributes.Emplace(TEXT("Context"), Context);

		// Gather Virtualization analytics
		UE::Virtualization::IVirtualizationSystem::Get().GatherAnalytics(EventAttributes);

		FStudioTelemetry::Get().RecordEvent(TEXT("Core.VirtualAssets"), EventAttributes);
	}
}

void FEditorTelemetry::RecordEvent_ShaderStats(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	const int SchemaVersion = 1;

	TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

	EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
	EventAttributes.Emplace(TEXT("Context"), Context);

	FShaderStatsFunctions::GatherShaderAnalytics(EventAttributes);

	FStudioTelemetry::Get().RecordEvent(TEXT("Core.ShaderStats"), EventAttributes);
}

void FEditorTelemetry::RecordEvent_MemoryLLM(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (FStudioTelemetry::Get().IsSessionRunning())
	{
		auto RecordLLMMemoryEvent = [&Attributes](const FString& Context, const FString& TagSet, TMap<FName, uint64>& LLMTrackedMemoryMap)
			{
				for (TMap<FName, uint64>::TConstIterator It(LLMTrackedMemoryMap); It; ++It)
				{
					const int SchemaVersion = 2;

					TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

					EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
					EventAttributes.Emplace(TEXT("Context"), Context);
					EventAttributes.Emplace(TEXT("TagSet"), TagSet);
					EventAttributes.Emplace(TEXT("Name"), It->Key);
					EventAttributes.Emplace(TEXT("Size"), It->Value);

					FStudioTelemetry::Get().RecordEvent("Core.Memory.LLM", EventAttributes);
				}
			};

		// None TagSet
		TMap<FName, uint64> LLMTrackedNoneMemory;
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(LLMTrackedNoneMemory, ELLMTracker::Default, ELLMTagSet::None);
		RecordLLMMemoryEvent(Context, TEXT("None"), LLMTrackedNoneMemory);

		// AssetClasses TagSet
		TMap<FName, uint64> LLMTrackedAssetClassesMemory;
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(LLMTrackedAssetClassesMemory, ELLMTracker::Default, ELLMTagSet::AssetClasses);
		RecordLLMMemoryEvent(Context, TEXT("AssetClasses"), LLMTrackedAssetClassesMemory);

		// Asset TagSet	
		TMap<FName, uint64> LLMTrackedAssetMemory;
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(LLMTrackedAssetMemory, ELLMTracker::Default, ELLMTagSet::Assets);
		RecordLLMMemoryEvent(Context, TEXT("Assets"), LLMTrackedAssetMemory);
	}
#endif
}

void FEditorTelemetry::RecordEvent_CoreSystems(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	FEditorTelemetry::RecordEvent_DDCResource(Context, Attributes);
	FEditorTelemetry::RecordEvent_DDCSummary(Context, Attributes);
	FEditorTelemetry::RecordEvent_Zen(Context, Attributes);
	FEditorTelemetry::RecordEvent_VirtualAssets(Context, Attributes);
	FEditorTelemetry::RecordEvent_MemoryLLM(Context, Attributes);
}

void FEditorTelemetry::RegisterCollectionWorkflowDelegates(FTelemetryRouter& Router)
{
	Router.OnTelemetry<FAssetAddedToCollectionTelemetryEvent>([](const FAssetAddedToCollectionTelemetryEvent& Event)
	{
		const int SchemaVersion = 1;
		
		FStudioTelemetry::Get().RecordEvent(TEXT("Editor.Collections.AssetsAdded"),
		{
			{ TEXT("SchemaVersion"), SchemaVersion},
			{ TEXT("DurationSec"), Event.DurationSec},
			{ TEXT("ObjectCount"), Event.NumAdded},
			{ TEXT("Workflow"), Event.Workflow},
			{ TEXT("CollectionShareType"), ECollectionShareType::ToString(Event.CollectionShareType)},
		});
	});

	Router.OnTelemetry<FAssetRemovedFromCollectionTelemetryEvent>([](const FAssetRemovedFromCollectionTelemetryEvent& Event)
	{
		const int SchemaVersion = 1;
		
		FStudioTelemetry::Get().RecordEvent(TEXT("Editor.Collections.AssetsRemoved"),
		{
			{ TEXT("SchemaVersion"), SchemaVersion},
			{ TEXT("DurationSec"), Event.DurationSec},
			{ TEXT("ObjectCount"), Event.NumRemoved},
			{ TEXT("Workflow"), Event.Workflow},
			{ TEXT("CollectionShareType"), ECollectionShareType::ToString(Event.CollectionShareType) },
		});
	});

	Router.OnTelemetry<FCollectionCreatedTelemetryEvent>([](const FCollectionCreatedTelemetryEvent& Event)
	{
		const int SchemaVersion = 1;

		FStudioTelemetry::Get().RecordEvent(TEXT("Editor.Collections.CollectionCreated"),
		{
			{ TEXT("SchemaVersion"), SchemaVersion},
			{ TEXT("DurationSec"), Event.DurationSec},
			{ TEXT("CollectionShareType"), ECollectionShareType::ToString(Event.CollectionShareType)},
		});
	});

	Router.OnTelemetry<FCollectionsDeletedTelemetryEvent>([](const FCollectionsDeletedTelemetryEvent& Event)
	{
		const int SchemaVersion = 1;

		FStudioTelemetry::Get().RecordEvent(TEXT("Editor.Collections.CollectionDeleted"),
		{
			{ TEXT("SchemaVersion"), SchemaVersion},
			{ TEXT("DurationSec"), Event.DurationSec},
			{ TEXT("ObjectCount"), Event.CollectionsDeleted},
		});
	});
}

extern ENGINE_API float GAverageFPS;

void FEditorTelemetry::HitchSamplerCallback()
{
	// Only sample framerate when we have focus
	if (FApp::HasFocus())
	{
		// Sample a rolling average of FPS 
		HitchAvergageFPS = (HitchAvergageFPS * HitchSampleCount + GAverageFPS) / (double)(HitchSampleCount + 1);
		HitchSampleCount++;
	}
}
	
void FEditorTelemetry::HeartbeatCallback()
{
	if (HitchSampleCount>0)
	{
		// Hitching is when FPS is below our threshold
		const bool IsHitching = HitchAvergageFPS < MinFPSForHitching;
		static uint32 HitchCount = 0;
		HitchCount += IsHitching ? 1 : 0;

		if (IsHitching == false && HitchingSpan.IsValid() == true)
		{
			// No longer hitching and we have started a hitch span
			TArray<FAnalyticsEventAttribute> Attributes;
			const double ElapsedTime = HitchingSpan->GetElapsedTime();
			Attributes.Emplace(FAnalyticsEventAttribute(TEXT("Hitch_Count"), HitchCount));
			Attributes.Emplace(FAnalyticsEventAttribute(TEXT("Hitch_HitchesPerSecond"), ElapsedTime>0? (float)HitchCount / ElapsedTime : 0.0));
			Attributes.Emplace(FAnalyticsEventAttribute(TEXT("Hitch_AverageFPS"), HitchAvergageFPS));
			Attributes.Emplace(TEXT("MapName"), EditorMapName);
			Attributes.Emplace(TEXT("PIE_MapName"), PIEMapName);

			// End the hitch Span
			FStudioTelemetry::Get().EndSpan(HitchingSpan, Attributes);

			// Record the hitch event
			FStudioTelemetry::Get().RecordEvent(TEXT("Core.Hitch"), Attributes);

			// Record core systems events for the hitch
			RecordEvent_CoreSystems(TEXT("Hitch"));

			// No longer need the hitch span for now so reset it
			HitchingSpan.Reset();
		}
		else if (IsHitching == true && HitchingSpan.IsValid() == false)
		{
			// We are hitching and we have not started a hitch span
			HitchingSpan = FStudioTelemetry::Get().StartSpan(HitchingSpanName);
			HitchCount = 1;
		}

		// Reset the hitch sampler
		HitchSampleCount = 0;
		HitchAvergageFPS = 0;
	}
}

void FEditorTelemetry::StartSession()
{
	if (FStudioTelemetry::Get().IsSessionRunning() == false)
	{
		return;
	}

	SessionStartTime = FPlatformTime::Seconds();

	// Install Editor Only Mode callbacks. Do not record these for Editor Commandlet runs 
	if (GIsEditor==true && IsRunningCommandlet()==false)
	{
		// Start Editor and Editor Boot span. Note : this will only start when the plugin is loaded and as such will miss any activity that runs beforehand
		EditorSpan = FStudioTelemetry::Get().StartSpan(EditorSpanName);
		EditorBootSpan = FStudioTelemetry::Get().StartSpan(EditorBootSpanName, EditorSpan);
		EditorMapName = TEXT("None");

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(FAnalyticsEventAttribute(TEXT("MapName"), EditorMapName));
		EditorBootSpan->AddAttributes(Attributes);
	
		FEditorDelegates::OnEditorBoot.AddLambda([this](double)
			{
				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Emplace(TEXT("PluginCount"), TotalPluginCount);
				EditorBootSpan->AddAttributes(Attributes);

				FStudioTelemetry::Get().EndSpan(EditorBootSpan);

				// Callback is received when the editor has booted but has not been initialized
				FEditorTelemetry::RecordEvent_Loading(TEXT("Editor.Boot"), EditorBootSpan->GetDuration(), EditorBootSpan->GetAttributes());
				FEditorTelemetry::RecordEvent_CoreSystems(TEXT("Editor.Boot"), EditorBootSpan->GetAttributes());
				
				EditorInitilizeSpan = FStudioTelemetry::Get().StartSpan(EditorInitilizeSpanName, EditorSpan);
			});

		FEditorDelegates::OnEditorInitialized.AddLambda([this](double TimeToInitializeEditor)
			{
				TimeToBootEditor = TimeToInitializeEditor;

				// Editor has initialized
				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Emplace(TEXT("MapName"), EditorMapName);
				Attributes.Emplace(TEXT("PluginCount"), TotalPluginCount);
				EditorInitilizeSpan->AddAttributes(Attributes);
				
				// Editor has finished initializing so start the Editor Interact span
				FStudioTelemetry::Get().EndSpan(EditorInitilizeSpan);

				FEditorTelemetry::RecordEvent_Loading(TEXT("Editor.Initialize"), EditorInitilizeSpan->GetDuration(), EditorInitilizeSpan->GetAttributes());
				FEditorTelemetry::RecordEvent_CoreSystems(TEXT("Editor.Initialize"), EditorInitilizeSpan->GetAttributes());
				FEditorTelemetry::RecordEvent_Loading(TEXT("TimeToEdit"), TimeToInitializeEditor, EditorInitilizeSpan->GetAttributes());
				
				EditorInteractSpan = FStudioTelemetry::Get().StartSpan(EditorInteractSpanName, EditorSpan);

				
				// Install callbacks for Open Asset Dialogue
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestedOpen().AddLambda([this](UObject* Asset)
					{
						AssetOpenStartTime = FPlatformTime::Seconds();
						FStudioTelemetry::Get().StartSpan(OpenAssetEditorSpan);
					});

				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetOpenedInEditor().AddLambda([this](UObject* Asset, IAssetEditorInstance*)
					{
						TArray<FAnalyticsEventAttribute> Attributes;
						Attributes.Emplace(TEXT("MapName"), EditorMapName);

						if (Asset != nullptr)
						{
							Attributes.Emplace(TEXT("AssetPath"), Asset->GetFullName());
							Attributes.Emplace(TEXT("AssetClass"), Asset->GetClass()->GetName());
						}

						FStudioTelemetry::Get().EndSpan(OpenAssetEditorSpan, Attributes);
					});

				// Setup a timer for a Heartbeat callback
				FTimerDelegate HeartbeatDelegate;
				HeartbeatDelegate.BindRaw(this, &FEditorTelemetry::HeartbeatCallback);
				GEditor->GetTimerManager()->SetTimer(TelemetryHeartbeatTimerHandle, HeartbeatDelegate, HeartbeatIntervalSeconds, true);

				// Setup the timer for the Hitch Detector callback
				FTimerDelegate HitchSamplerDelegate;
				HitchSamplerDelegate.BindRaw(this, &FEditorTelemetry::HitchSamplerCallback);
				GEditor->GetTimerManager()->SetTimer(TelemetryHitchSamplerTimerHandle, HitchSamplerDelegate, HitchSamplerIntervalSeconds, true);
			});

		// Install PIE Mode callbacks
		FEditorDelegates::StartPIE.AddLambda([this](bool)
			{
				// PIE mode has been started. The user has pressed the Start PIE button.
				// Finish the Editor span
				FStudioTelemetry::Get().EndSpan(EditorSpan);
				EditorSessionCount++;

				// Start PIE span
				PIESpan = FStudioTelemetry::Get().StartSpan(PIESpanName);

				// Append the PIE transition count to the PIE name
				PIEStartupSpan = FStudioTelemetry::Get().StartSpan(PIESessionCount == 0 ? PIEStartupSpanName : FName(*FString::Printf(TEXT("%s.Iterate"), *PIEStartupSpanName.ToString())), PIESpan);

				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Emplace(TEXT("MapName"), EditorMapName);

				PIESpan->AddAttributes(Attributes);
				PIEStartupSpan->AddAttributes(Attributes);
			});

		FEditorDelegates::PreBeginPIE.AddLambda([this](bool)
			{
				PIEPreBeginSpan = FStudioTelemetry::Get().StartSpan(PIESessionCount == 0 ? PIEPreBeginSpanName : FName(*FString::Printf(TEXT("%s.Iterate"), *PIEPreBeginSpanName.ToString())), PIESpan);
				PIEPreBeginSpan->AddAttributes(PIESpan->GetAttributes());
			});

		FEditorDelegates::BeginPIE.AddLambda([this](bool)
			{
				FStudioTelemetry::Get().EndSpan(PIEPreBeginSpan);
			});

		FWorldDelegates::OnPIEMapCreated.AddLambda([this](UGameInstance* GameInstance)
			{
				// A new PIE map was created
				PIELoadMapSpan = FStudioTelemetry::Get().StartSpan(PIELoadMapSpanName, PIEStartupSpan);
			});

		FWorldDelegates::OnPIEMapReady.AddLambda([this](UGameInstance* GameInstance)
			{
				// PIE map is now loaded and ready to use
				PIEMapName = FPaths::GetBaseFilename(GameInstance->PIEMapName);

				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Emplace(TEXT("MapName"), EditorMapName);
				Attributes.Emplace(TEXT("PIE_MapName"), PIEMapName);

				PIELoadMapSpan->AddAttributes(Attributes);

				FStudioTelemetry::Get().EndSpan(PIELoadMapSpan);

				FEditorTelemetry::RecordEvent_Loading(TEXT("PIE.LoadMap"), PIELoadMapSpan->GetDuration(), PIELoadMapSpan->GetAttributes());
				FEditorTelemetry::RecordEvent_CoreSystems(TEXT("PIE.LoadMap"), PIELoadMapSpan->GetAttributes());
			});

		FWorldDelegates::OnPIEReady.AddLambda([this](UGameInstance* GameInstance)
			{
				if (PIESpan.IsValid())
				{
					if (PIEStartupSpan.IsValid())
					{
						// PIE is now ready for user interaction

						// Keep track of the PIE transition counts
						TArray<FAnalyticsEventAttribute> Attributes;
						Attributes.Emplace(TEXT("PIE_TransitionCount"), PIESessionCount);

						PIESpan->AddAttributes(Attributes);
						PIEStartupSpan->AddAttributes(Attributes);

						FStudioTelemetry::Get().EndSpan(PIEStartupSpan);

						// Record the PIE startup
						FEditorTelemetry::RecordEvent_Loading(TEXT("PIE.Startup"), PIEStartupSpan->GetDuration(), PIEStartupSpan->GetAttributes());
						FEditorTelemetry::RecordEvent_CoreSystems(TEXT("PIE.Startup"), PIEStartupSpan->GetAttributes());

						// Record the time from start PIE to PIE
						if (PIESessionCount == 0)
						{
							const double TimeInEditor = EditorLoadMapSpan.IsValid() ? EditorLoadMapSpan->GetDuration() : 0.0;
							const double TimeToStartPIE = PIEStartupSpan->GetDuration();
							const double TimeToBootToPIE = TimeToBootEditor + TimeInEditor + TimeToStartPIE;

							// Record the absolute time from editor boot to PIE
							FEditorTelemetry::RecordEvent_Loading(TEXT("TimeToPIE"), TimeToBootToPIE, PIEStartupSpan->GetAttributes());	
						}
					}

					PIEInteractSpan = FStudioTelemetry::Get().StartSpan(PIESessionCount == 0 ? PIEInteractSpanName : FName(*FString::Printf(TEXT("%s.Iterate"), *PIEInteractSpanName.ToString())), PIESpan);
					PIEInteractSpan->AddAttributes(PIESpan->GetAttributes());
				}
			});

		FEditorDelegates::EndPIE.AddLambda([this](bool)
			{
				if (PIESpan.IsValid())
				{
					// PIE is ending so no longer interactive
					FStudioTelemetry::Get().EndSpan(PIEInteractSpan);
					PIEShutdownSpan = FStudioTelemetry::Get().StartSpan(PIESessionCount == 0 ? PIEShutdownSpanName : FName(*FString::Printf(TEXT("%s.Iterate"), *PIEShutdownSpanName.ToString())), PIESpan);
					PIEShutdownSpan->AddAttributes(PIESpan->GetAttributes());
				}
			});

		FEditorDelegates::ShutdownPIE.AddLambda([this](bool)
			{
				if (PIESpan.IsValid())
				{
					// PIE has shutdown, ie. the user has pressed the Stop PIE button, and we are going back to interactive Editor mode	
					FStudioTelemetry::Get().EndSpan(PIEShutdownSpan);
					FEditorTelemetry::RecordEvent_Loading(TEXT("PIE.Shutdown"), PIEShutdownSpan->GetDuration(), PIEShutdownSpan->GetAttributes());
					FEditorTelemetry::RecordEvent_CoreSystems(TEXT("PIE.Shutdown"), PIEShutdownSpan->GetAttributes());
					
					FStudioTelemetry::Get().EndSpan(PIESpan);
				}

				PIESessionCount++;

				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Emplace(TEXT("MapName"), EditorMapName);

				// Restart the Editor span
				EditorSpan = FStudioTelemetry::Get().StartSpan(EditorSpanName, Attributes);
				EditorInteractSpan = FStudioTelemetry::Get().StartSpan(FName(*FString::Printf(TEXT("%s%d"), *EditorInteractSpanName.ToString(), EditorSessionCount)), EditorSpan);
			});

		FEditorDelegates::OnMapLoad.AddLambda([this](const FString& MapName, FCanLoadMap& OutCanLoadMap)
			{
				if (MapName.Len() > 0)
				{
					// The Editor loads a new map
					EditorLoadMapSpan = FStudioTelemetry::Get().StartSpan(EditorLoadMapSpanName, EditorSpan);
				}
			});

		FEditorDelegates::OnMapOpened.AddLambda([this](const FString& MapName, bool Unused)
			{
				if (EditorLoadMapSpan.IsValid())
				{
					// The new editor map was actually opened
					EditorMapName = FPaths::GetBaseFilename(MapName);

					TArray<FAnalyticsEventAttribute> Attributes;
					Attributes.Emplace(TEXT("MapName"), EditorMapName);

					EditorSpan->AddAttributes(Attributes);
					EditorLoadMapSpan->AddAttributes(Attributes);

					FStudioTelemetry::Get().EndSpan(EditorLoadMapSpan);

					FEditorTelemetry::RecordEvent_Loading(TEXT("Editor.LoadMap"), EditorLoadMapSpan->GetDuration(), EditorLoadMapSpan->GetAttributes());
					FEditorTelemetry::RecordEvent_CoreSystems(TEXT("Editor.LoadMap"), EditorLoadMapSpan->GetAttributes());

					EditorLoadMapSpan.Reset();
				}
			});
	}

	// Install any plugin load/unload callbacks
	FModuleManager::Get().OnModulesChanged().AddLambda([this](FName ModuleName, EModuleChangeReason ChangeReason)
		{
			switch (ChangeReason)
			{
				default:
				{
					break;
				}

				case EModuleChangeReason::ModuleLoaded:
				{
					TotalPluginCount++;

					// Hook into Asset Registry Scan callbacks as as soon as it is loaded
					if (ModuleName == TEXT("AssetRegistry"))
					{
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

						AssetRegistryModule.Get().OnScanStarted().AddLambda([this]()
							{
								if (AssetRegistryScanCount == 0)
								{
									// Start the Asset Registry Scan span
									AssetRegistryScanSpan = FStudioTelemetry::Get().StartSpan(AssetRegistryScanSpanName, EditorSpan);
								}

								AssetRegistryScanCount++;
							});

						AssetRegistryModule.Get().OnScanEnded().AddLambda([this]()
							{
								AssetRegistryScanCount--;

								if (AssetRegistryScanCount == 0)
								{
									// End the Asset Registry Scan span
									TArray<FAnalyticsEventAttribute> Attributes;
									Attributes.Emplace(TEXT("MapName"), EditorMapName);
									FStudioTelemetry::Get().EndSpan(AssetRegistryScanSpan, Attributes);
								}
							});
					}

					break;
				}

				case EModuleChangeReason::ModuleUnloaded:
				{
					TotalPluginCount--;
					break;
				}
			}
		});
	
	// Set up Slow Task callbacks
	ensureMsgf(GWarn, TEXT("GWarn was not valid"));

	if (GWarn != nullptr)
	{
		// Start the SlowTask span
		GWarn->OnStartSlowTaskWithGuid().AddLambda([this](FGuid TaskGuid, const FText& TaskName)
			{
				// Slow tasks can possibly be started from multiple threads, so we need to protect the registered span table
				FScopeLock ScopeLock(&TaskSpanCriticalSection);

				TSharedPtr<IAnalyticsSpan>* SpanPtr = TaskSpans.Find(TaskGuid);

				// Only one task with this Guid is running asynchronously is supported at this time.
				if (SpanPtr == nullptr)
				{
					TArray<FAnalyticsEventAttribute> Attributes;
					Attributes.Emplace(TEXT("MapName"), EditorMapName);
					Attributes.Emplace(TEXT("TaskName"), TaskName.ToString());

					// Create and start a new slow task span
					TSharedPtr<IAnalyticsSpan> SlowTaskSpan = FStudioTelemetry::Get().StartSpan(TEXT("SlowTask"), Attributes);

					// Store this SlowTask span so we can find it when it finishes
					TaskSpans.Add(TaskGuid, SlowTaskSpan);
				}

				TRACE_BEGIN_REGION(*TaskName.ToString());
			});

		// End the SlowTask span
		GWarn->OnFinalizeSlowTaskWithGuid().AddLambda([this](FGuid TaskGuid, const FText& TaskName)
			{
				TRACE_END_REGION(*TaskName.ToString());

				// Slow tasks can possibly be finalized from multiple threads, so we need to protect the registered span table
				FScopeLock ScopeLock(&TaskSpanCriticalSection);

				// Find the task we stored off when we started this task
				TSharedPtr<IAnalyticsSpan>* SpanPtr = TaskSpans.Find(TaskGuid);

				if (SpanPtr != nullptr)
				{
					TSharedPtr<IAnalyticsSpan> SlowTaskSpan = *SpanPtr;

					FStudioTelemetry::Get().EndSpan(SlowTaskSpan);

					// Remove the SlowTask span from the registry
					TaskSpans.Remove(TaskGuid);
				}
			});
	}

	// Install Cooking Callbacks
	if (GUnrealEd != nullptr && GUnrealEd->CookServer != nullptr)
	{
		UE::Cook::FDelegates::CookFinished.AddLambda([this](UE::Cook::ICookInfo& CookInfo)
			{
				if (CookInfo.GetCookType() != UE::Cook::ECookType::ByTheBook)
				{
					return;
				}
				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Emplace(TEXT("MapName"), EditorMapName);

				FEditorTelemetry::RecordEvent_Cooking(TEXT("Cook.End"), Attributes);
				FEditorTelemetry::RecordEvent_CoreSystems(TEXT("Cook.End"), Attributes);
				FEditorTelemetry::RecordEvent_DDCVerification(TEXT("Cook.End"), Attributes);
				FEditorTelemetry::RecordEvent_ShaderStats(TEXT("Cook.End"), CookingSpan->GetAttributes());
			});
	}

	UE::Cook::FDelegates::CookStarted.AddLambda([this](UE::Cook::ICookInfo& CookInfo)
	{
		if (CookInfo.GetCookType() != UE::Cook::ECookType::ByTheBook)
		{
			return;
		}
		// Begin the cooking span	
		CookingSpan = FStudioTelemetry::Get().StartSpan(TEXT("Cooking"));

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("MapName"), EditorMapName);

		CookingSpan->AddAttributes(Attributes);

		const double TimeToCook = FPlatformTime::Seconds() - SessionStartTime;
		FEditorTelemetry::RecordEvent_Loading(TEXT("Cook.Start"), TimeToCook, CookingSpan->GetAttributes());
	});

	UE::Cook::FDelegates::CookFinished.AddLambda([this](UE::Cook::ICookInfo& CookInfo)
	{
		if (CookInfo.GetCookType() != UE::Cook::ECookType::ByTheBook)
		{
			return;
		}
	
		uint32 MultiprocessId = 0;
		FParse::Value(FCommandLine::Get(), TEXT("-MultiprocessId="), MultiprocessId);
		const bool bIsCookWorker = MultiprocessId != 0;

		if (bIsCookWorker)
		{
			// Cook-worker specific telemetry goes here
			FEditorTelemetry::RecordEvent_DDCVerification(TEXT("Cook.End"), CookingSpan->GetAttributes());
		}
		else
		{
			// Cook director/single-process cook telemetry goes here
			FEditorTelemetry::RecordEvent_Cooking(TEXT("Cook.End"), CookingSpan->GetAttributes());
			FEditorTelemetry::RecordEvent_CoreSystems(TEXT("Cook.End"), CookingSpan->GetAttributes());
			FEditorTelemetry::RecordEvent_DDCVerification(TEXT("Cook.End"), CookingSpan->GetAttributes());
			FEditorTelemetry::RecordEvent_ShaderStats(TEXT("Cook.End"), CookingSpan->GetAttributes());

			// End the cooking span
			FStudioTelemetry::Get().EndSpan(CookingSpan);
		}
	});

	// Install Content Browser callbacks
	FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>( TEXT("ContentBrowser") );
	
	FTelemetryRouter& Router = FTelemetryRouter::Get();
	{
		using namespace UE::Telemetry::ContentBrowser;
		Router.OnTelemetry<FBackendFilterTelemetry>([this](const FBackendFilterTelemetry& Data)
		{
			FString DataFilterText = LexToString(FJsonNull{});
			if (Data.DataFilter)
			{
				Private::FAnalyticsJsonWriter J(&DataFilterText);
				J.WriteObjectStart();
				J.WriteValue("RecursivePaths", Data.DataFilter->bRecursivePaths);
				J.WriteValue("ItemTypeFilter", UEnum::GetValueOrBitfieldAsString(Data.DataFilter->ItemTypeFilter));
				J.WriteValue("ItemCategoryFilter", UEnum::GetValueOrBitfieldAsString(Data.DataFilter->ItemCategoryFilter));
				J.WriteValue("ItemAttributeFilter", UEnum::GetValueOrBitfieldAsString(Data.DataFilter->ItemAttributeFilter));
				TArray<const UScriptStruct*> FilterTypes = Data.DataFilter->ExtraFilters.GetFilterTypes();
				if (FilterTypes.Num() > 0)
				{
					J.WriteArrayStart("FilterTypes");
					for (const UScriptStruct* Type : FilterTypes)
					{
						J.WriteValue(Type->GetPathName());
					}
					J.WriteArrayEnd();				
				}
				J.WriteObjectEnd();
				J.Close();
			}
			
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetView.BackendFilter"), 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("ViewCorrelationGuid"), Data.ViewCorrelationGuid },
				{ TEXT("FilterSessionCorrelationGuid"), Data.FilterSessionCorrelationGuid },
				{ TEXT("HasCustomItemSources"), Data.bHasCustomItemSources },
				{ TEXT("RefreshSourceItemsDurationSeconds"), Data.RefreshSourceItemsDurationSeconds },
				{ TEXT("NumBackendItems"), Data.NumBackendItems },
				{ TEXT("DataFilter"), FJsonFragment(MoveTemp(DataFilterText)) },
			});
		});

		Router.OnTelemetry<FFrontendFilterTelemetry>([this](const FFrontendFilterTelemetry& Data)
		{
			FString FilterText = LexToString(FJsonNull{});
			if (Data.FrontendFilters.IsValid() && Data.FrontendFilters->Num())
			{
				Private::FAnalyticsJsonWriter J(&FilterText);
				J.WriteArrayStart();
				for (const TSharedPtr<IFilter<FAssetFilterType>>& Filter : *Data.FrontendFilters.Get())
				{
					J.WriteValue(Filter->GetName());
				}
				J.WriteArrayEnd();
				J.Close();
			}
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetView.FrontendFilter"), 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("ViewCorrelationGuid"), Data.ViewCorrelationGuid },
				{ TEXT("FilterSessionCorrelationGuid"), Data.FilterSessionCorrelationGuid },
				{ TEXT("TotalItemsToFilter"), Data.TotalItemsToFilter },
				{ TEXT("PriorityItemsToFilter"), Data.PriorityItemsToFilter },
				{ TEXT("TotalResults"), Data.TotalResults },
				{ TEXT("AmortizeDurationSeconds"), Data.AmortizeDuration },
				{ TEXT("WorkDurationSeconds"), Data.WorkDuration },
				{ TEXT("ResultLatency"), AnalyticsOptionalToStringOrNull(Data.ResultLatency) },
				{ TEXT("TimeUntilInteractionSeconds"), AnalyticsOptionalToStringOrNull(Data.TimeUntilInteraction) },
				{ TEXT("Completed"), Data.bCompleted },
				{ TEXT("FrontendFilters"), FJsonFragment(MoveTemp(FilterText)) },
			});
		});

		RegisterCollectionWorkflowDelegates(Router);
	}
	{
		using namespace UE::Telemetry::AssetRegistry;

		Router.OnTelemetry<FStartupTelemetry>([this](const FStartupTelemetry& Data){
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetRegistry.Startup"), 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("Duration"), Data.StartupDuration },
				{ TEXT("StartedAsyncGather"), Data.bStartedAsyncGather },
			});
		});
		Router.OnTelemetry<FSynchronousScanTelemetry>([this](const FSynchronousScanTelemetry& Data){
			if (Data.Duration < 0.5)
			{
				return;
			}
			FString DirectoriesText;
			{
				Private::FAnalyticsJsonWriter J(&DirectoriesText);
				J.WriteArrayStart();
				for (const FString& Directory : MakeArrayView(Data.Directories).Left(100))
				{
					J.WriteValue(Directory);
				}
				J.WriteArrayEnd();
				J.Close();
			}
			FString FilesText;
			{
				Private::FAnalyticsJsonWriter J(&FilesText);
				J.WriteArrayStart();
				for (const FString& File : MakeArrayView(Data.Files).Left(100))
				{
					J.WriteValue(File);
				}
				J.WriteArrayEnd();
				J.Close();
			}
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetRegistry.SynchronousScan"), 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("Directories"), FJsonFragment(MoveTemp(DirectoriesText)) },
				{ TEXT("Files"), FJsonFragment(MoveTemp(FilesText)) },
				{ TEXT("Flags"), LexToString(Data.Flags) },
				{ TEXT("NumFoundAssets"), Data.NumFoundAssets },
				{ TEXT("DurationSeconds"), Data.Duration },
				{ TEXT("InitialSearchStarted"), Data.bInitialSearchStarted },
				{ TEXT("InitialSearchCompleted"), Data.bInitialSearchCompleted },
				{ TEXT("AdditionalMountSearchInProgress"), Data.bAdditionalMountSearchInProgress },
			});
		});
		Router.OnTelemetry<FGatherTelemetry>([this](const FGatherTelemetry& Data){
			const TCHAR* EventName = Data.bIsInitialScan ? TEXT("Editor.AssetRegistry.InitialScan")  : TEXT("Editor.AssetRegistry.AdditionalScan");
			FStudioTelemetry::Get().RecordEvent(EventName, 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("TotalDurationSeconds"), Data.TotalSearchDurationSeconds },
				{ TEXT("TotalWorkSeconds"), Data.TotalWorkTimeSeconds },
				{ TEXT("DiscoverySeconds"), Data.DiscoveryTimeSeconds },
				{ TEXT("GatherSeconds"), Data.GatherTimeSeconds },
				{ TEXT("StoreSeconds"), Data.StoreTimeSeconds },
				{ TEXT("NumCachedDirectories"), Data.NumCachedDirectories },
				{ TEXT("NumUncachedDirectories"), Data.NumUncachedDirectories },
				{ TEXT("NumCachedAssetFiles"), Data.NumCachedAssetFiles },
				{ TEXT("NumUncachedAssetFiles"), Data.NumUncachedAssetFiles },
			});
		});
		Router.OnTelemetry<FDirectoryWatcherUpdateTelemetry>([this](const FDirectoryWatcherUpdateTelemetry& Data){
			if (Data.DurationSeconds < 0.5)
			{
				return;
			}
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetRegistry.DirectoryWatcherUpdate"), 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("NumChanges"), Data.Changes.Num() },
				{ TEXT("DurationSeconds"), Data.DurationSeconds },
				{ TEXT("InitialSearchStarted"), Data.bInitialSearchStarted },
				{ TEXT("InitialSearchCompleted"), Data.bInitialSearchCompleted },
			});
		});
		Router.OnTelemetry<FFileJournalErrorTelemetry>([this](const FFileJournalErrorTelemetry& Data){
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetRegistry.FileJournalError"),
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("Directory"), Data.Directory },
				{ TEXT("ErrorString"), Data.ErrorString },
			});
		});
		Router.OnTelemetry<FFileJournalWrappedTelemetry>([this](const FFileJournalWrappedTelemetry& Data) {
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.AssetRegistry.FileJournalWrapped"),
				{
					{ TEXT("SchemaVersion"), 1 },
					{ TEXT("VolumeName"), Data.VolumeName},
					{ TEXT("JournalMaximumSize"), Data.JournalMaximumSize},
				});
			});
	}

	// Data validation telemetry
	{
		using namespace UE::Telemetry;
		using namespace UE::Telemetry::DataValidation;

		Router.OnTelemetry<FValidateAssets>([this](const FValidateAssets& Data){
			FString ValidatorStats;
			{
				::Private::FAnalyticsJsonWriter J(&ValidatorStats);
				J.WriteObjectStart();
				TArray<TPair<FTopLevelAssetPath, FValidatorStatistics>> Stats;
				Algo::Copy(Data.ValidatorStatistics, Stats);
				Algo::SortBy(Stats, 
					[](const TPair<FTopLevelAssetPath, FValidatorStatistics>& Pair) { return Pair.Value.AssetsValidated + Pair.Value.AssetsAddedForValidation; },
					TGreater<int32>{});
				for (const TPair<FTopLevelAssetPath, FValidatorStatistics>& Pair : MakeArrayView(Stats).Left(10))
				{
					J.WriteObjectStart(WriteToString<256>(Pair.Key).ToView());
					J.WriteValue(TEXT("NumAssetsValidated"), Pair.Value.AssetsValidated);
					J.WriteValue(TEXT("NumAssetsAddedForValidation"), Pair.Value.AssetsAddedForValidation);
					J.WriteObjectEnd();
				}
				J.WriteObjectEnd();
				J.Close();
			}
			FStudioTelemetry::Get().RecordEvent(TEXT("Editor.DataValidation.ValidateAssets"), 
			{
				{ TEXT("SchemaVersion"), 1 },
				{ TEXT("Duration"), Data.Duration },
				{ TEXT("Mode"), LexToString(Data.Mode) },
				{ TEXT("Result"), LexToString(Data.Result) },
				{ TEXT("SkipExcludedDirectories"), Data.bSkipExcludedDirectories },
				{ TEXT("ValidationUsecase"), LexToString(Data.ValidationUsecase) },
				{ TEXT("LoadAssetsForValidation"), Data.bLoadAssetsForValidation },
				{ TEXT("UnloadAssetsLoadedForValidation"), Data.bUnloadAssetsLoadedForValidation },
				{ TEXT("LoadExternalObjectsForValidation"), Data.bLoadExternalObjectsForValidation },
				{ TEXT("CaptureAssetLoadLogs"), Data.bCaptureAssetLoadLogs },
				{ TEXT("CaptureLogsDuringValidation"), Data.bCaptureLogsDuringValidation },
				{ TEXT("CaptureWarningsDuringValidationAsErrors"), Data.bCaptureWarningsDuringValidationAsErrors },
				{ TEXT("MaxAssetsToValidate"), Data.MaxAssetsToValidate },
				{ TEXT("ValidateReferencersOfDeletedAssets"), Data.bValidateReferencersOfDeletedAssets },
				{ TEXT("NumRequested"), Data.NumRequested },
				{ TEXT("NumExternalObjects"), Data.NumExternalObjects },
				{ TEXT("NumChecked"), Data.NumChecked },
				{ TEXT("NumValid"), Data.NumValid },
				{ TEXT("NumInvalid"), Data.NumInvalid },
				{ TEXT("NumSkipped"), Data.NumSkipped },
				{ TEXT("NumWarnings"), Data.NumWarnings },
				{ TEXT("NumUnableToValidate"), Data.NumUnableToValidate },
				{ TEXT("AssetLimitReached"), Data.bAssetLimitReached },
				{ TEXT("ValidatorStatistics"), FJsonFragment(MoveTemp(ValidatorStats)) },
			});
		});
	}

	{
		UE::Virtualization::GetAnalyticsRecordEvent().AddLambda([](const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, UE::Virtualization::EAnalyticsFlags Flags)
			{
				FStudioTelemetry::Get().RecordEvent(EventName, Attributes);

				if (EnumHasAllFlags(Flags, UE::Virtualization::EAnalyticsFlags::Flush))
				{
					FStudioTelemetry::Get().FlushEvents();
				}
			});
	}
}

void FEditorTelemetry::EndSession()
{
	if (EditorSpan.IsValid())
	{
		FStudioTelemetry::Get().EndSpan(EditorSpan);
		EditorSpan.Reset();
	}

}
