// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTelemetryModule.h"
#include "EditorTelemetry.h"
#include "StudioTelemetry.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FEditorTelemetryModule, EditorTelemetry);

void FEditorTelemetryModule::StartupModule()
{
#if !UE_BUILD_SHIPPING
	FStudioTelemetry::Get().StartSession();
	FEditorTelemetry::Get().StartSession();
#endif
}

void FEditorTelemetryModule::ShutdownModule()
{
#if !UE_BUILD_SHIPPING
	FEditorTelemetry::Get().EndSession();
	FStudioTelemetry::Get().EndSession();
#endif
}
