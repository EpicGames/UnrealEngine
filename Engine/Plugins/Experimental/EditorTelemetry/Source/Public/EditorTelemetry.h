// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsProvider.h"
#include "Interfaces/IAnalyticsTracer.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Engine/EngineTypes.h"

#define UE_API EDITORTELEMETRY_API

class FTelemetryRouter;
struct FTimerHandle;

/**
 * A class that implements a variety of pre-configured Core and Editor telemetry events that can be used to evaluate the efficiency of the most common developer workflows
 */
class FEditorTelemetry
{
public:

	FEditorTelemetry() = default;
	~FEditorTelemetry() = default;

	static UE_API FEditorTelemetry& Get();

	UE_API void StartSession();
	UE_API void EndSession();

	/** Useful event recording functions */
	UE_API void RecordEvent_Cooking(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_Loading(const FString& Context, double LoadingSeconds, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_CoreSystems(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_DDCResource(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_DDCSummary(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_DDCVerification(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_Zen(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_VirtualAssets(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_MemoryLLM(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});
	UE_API void RecordEvent_ShaderStats(const FString& Context, const TArray<FAnalyticsEventAttribute>& Attributes = {});

	UE_API void RegisterCollectionWorkflowDelegates(FTelemetryRouter& Router);

private:	

	UE_API void HitchSamplerCallback();
	UE_API void HeartbeatCallback();
	
	TSharedPtr<IAnalyticsSpan> EditorSpan;
	TSharedPtr<IAnalyticsSpan> EditorBootSpan;
	TSharedPtr<IAnalyticsSpan> EditorInteractSpan;
	TSharedPtr<IAnalyticsSpan> EditorInitilizeSpan;
	TSharedPtr<IAnalyticsSpan> EditorLoadMapSpan;
	TSharedPtr<IAnalyticsSpan> PIESpan;
	TSharedPtr<IAnalyticsSpan> PIEPreBeginSpan;
	TSharedPtr<IAnalyticsSpan> PIEStartupSpan;
	TSharedPtr<IAnalyticsSpan> PIELoadMapSpan;
	TSharedPtr<IAnalyticsSpan> PIEInteractSpan;		
	TSharedPtr<IAnalyticsSpan> PIEShutdownSpan;
	TSharedPtr<IAnalyticsSpan> CookingSpan;
	TSharedPtr<IAnalyticsSpan> HitchingSpan;
	TSharedPtr<IAnalyticsSpan> AssetRegistryScanSpan;
	
	const FName EditorSpanName = TEXT("Editor");
	const FName EditorBootSpanName = TEXT("Editor.Boot");
	const FName EditorInitilizeSpanName = TEXT("Editor.Initialize");
	const FName EditorInteractSpanName = TEXT("Editor.Interact");
	const FName EditorLoadMapSpanName = TEXT("Editor.LoadMap");
	const FName AssetRegistryScanSpanName = TEXT("Editor.AssetRegistryScan");
	const FName PIESpanName = TEXT("PIE");
	const FName PIEStartupSpanName = TEXT("PIE.Startup");
	const FName PIEPreBeginSpanName = TEXT("PIE.PreBegin");
	const FName PIELoadMapSpanName = TEXT("PIE.LoadMap");
	const FName PIEInteractSpanName = TEXT("PIE.Interact");
	const FName PIEShutdownSpanName = TEXT("PIE.Shutdown");
	const FName CookingSpanName = TEXT("Cooking");
	const FName HitchingSpanName = TEXT("Hitching");
	const FName OpenAssetEditorSpan = TEXT("Open Asset Editor");
	const float HeartbeatIntervalSeconds = 1.0;
	const float HitchSamplerIntervalSeconds = 0.1f;
	const float MinFPSForHitching = 5.0f;

	TMap<FGuid, TSharedPtr<IAnalyticsSpan>> TaskSpans;
	FCriticalSection TaskSpanCriticalSection;

	FTimerHandle TelemetryHeartbeatTimerHandle;
	FTimerHandle TelemetryHitchSamplerTimerHandle;
	FString EditorMapName;
	FString PIEMapName;
	uint32 EditorSessionCount = 0;
	uint32 PIESessionCount = 0;
	double SessionStartTime;
	double AssetOpenStartTime;
	double TimeToBootEditor;
	double HitchAvergageFPS = 0;
	uint32 HitchSampleCount = 0;
	uint32 TotalPluginCount = 0;
	uint32 AssetRegistryScanCount = 0;
};


#undef UE_API
